// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class OverlayEditor : ModuleRules
	{
		public OverlayEditor(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
                new string[] {
				    "Core",
                    "CoreUObject",
                    "Overlay",
                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[] {
					"EditorFramework",
					"UnrealEd",
                    "Engine",
                }
            );
			
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"AssetTools",
				}
			);

            PrivateIncludePathModuleNames.AddRange(
                new string[] {
                    "AssetTools",
                }
            );

            if (Target.bBuildEditor)
            {
                PrivateIncludePathModuleNames.Add("TargetPlatform");
            }
        }
	}
}