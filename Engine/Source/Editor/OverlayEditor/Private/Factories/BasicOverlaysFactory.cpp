// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/BasicOverlaysFactory.h"
#include "BasicOverlays.h"
#include "OverlaysImporter.h"

#include "Misc/Paths.h"

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicOverlaysFactory)


/* UBasicOverlaysFactory structors
 *****************************************************************************/

UBasicOverlaysFactory::UBasicOverlaysFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Formats.Add(TEXT("srt;SubRip Subtitles"));

	SupportedClass = UBasicOverlays::StaticClass();
	bEditorImport = true;
}


/* UFactory overrides
 *****************************************************************************/

bool UBasicOverlaysFactory::FactoryCanImport(const FString& Filename)
{
	FOverlaysImporter Importer;

	return Importer.OpenFile(Filename);
}

UObject* UBasicOverlaysFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UBasicOverlays* OverlayObject = NewObject<UBasicOverlays>(InParent, InClass, InName, Flags);

	FOverlaysImporter Importer;

	if (Importer.OpenFile(Filename))
	{
		Importer.ImportBasic(OverlayObject->Overlays);
	}

#if WITH_EDITORONLY_DATA
	if (OverlayObject->AssetImportData != nullptr)
	{
		OverlayObject->AssetImportData->Update(UFactory::GetCurrentFilename());
	}
#endif	// WITH_EDITORONLY_DATA

	return OverlayObject;
}
