// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Factories/Factory.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#include "BasicOverlaysFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;


/**
 * Implements a factory for UBasicOverlays objects.
 */
UCLASS(hidecategories=Object)
class UBasicOverlaysFactory
	: public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	//~ UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename);
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
};
