// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/BasicOverlaysFactoryNew.h"

#include "AssetTypeCategories.h"
#include "BasicOverlays.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicOverlaysFactoryNew)

class FFeedbackContext;
class UClass;
class UObject;


/* UBasicOverlaysFactoryNew structors
 *****************************************************************************/

UBasicOverlaysFactoryNew::UBasicOverlaysFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UBasicOverlays::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
 *****************************************************************************/

UObject* UBasicOverlaysFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UBasicOverlays>(InParent, InClass, InName, Flags);
}


uint32 UBasicOverlaysFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UBasicOverlaysFactoryNew::ShouldShowInNewMenu() const
{
	return false;// true;
}
