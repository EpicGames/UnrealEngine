// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/LocalizedOverlaysFactoryNew.h"

#include "AssetTypeCategories.h"
#include "LocalizedOverlays.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalizedOverlaysFactoryNew)

class FFeedbackContext;
class UClass;
class UObject;

/* ULocalizedOverlaysFactoryNew structors
*****************************************************************************/

ULocalizedOverlaysFactoryNew::ULocalizedOverlaysFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = ULocalizedOverlays::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory interface
*****************************************************************************/

UObject* ULocalizedOverlaysFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<ULocalizedOverlays>(InParent, InClass, InName, Flags);
}


uint32 ULocalizedOverlaysFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool ULocalizedOverlaysFactoryNew::ShouldShowInNewMenu() const
{
	return false;// true;
}
