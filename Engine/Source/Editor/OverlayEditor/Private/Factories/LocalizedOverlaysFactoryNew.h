// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#include "LocalizedOverlaysFactoryNew.generated.h"

class FFeedbackContext;
class UClass;
class UObject;

/**
* Implements a factory for new ULocalizedOverlays objects.
*/
UCLASS(hidecategories = Object)
class ULocalizedOverlaysFactoryNew
	: public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	//~ UFactory Interface

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};