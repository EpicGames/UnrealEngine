// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/ReimportBasicOverlaysFactory.h"
#include "BasicOverlays.h"
#include "OverlaysImporter.h"

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif	// WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReimportBasicOverlaysFactory)

UReimportBasicOverlaysFactory::UReimportBasicOverlaysFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UBasicOverlays::StaticClass();

	bCreateNew = false;
}

bool UReimportBasicOverlaysFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
#if WITH_EDITORONLY_DATA
	UBasicOverlays* Overlays = Cast<UBasicOverlays>(Obj);

	if (Overlays != nullptr && Overlays->AssetImportData != nullptr)
	{
		Overlays->AssetImportData->ExtractFilenames(OutFilenames);
		return true;
	}
#endif	// WITH_EDITORONLY_DATA

	return false;
}

void UReimportBasicOverlaysFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
#if WITH_EDITORONLY_DATA
	UBasicOverlays* Overlays = Cast<UBasicOverlays>(Obj);

	if (Overlays && Overlays->AssetImportData && ensure(NewReimportPaths.Num() == 1))
	{
		Overlays->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
#endif	// WITH_EDITORONLY_DATA
}

EReimportResult::Type UReimportBasicOverlaysFactory::Reimport(UObject* Obj)
{
	EReimportResult::Type ReimportResult = EReimportResult::Failed;

#if WITH_EDITORONLY_DATA

	UBasicOverlays* OverlayObject = Cast<UBasicOverlays>(Obj);
	FString Filename;

	if (OverlayObject != nullptr && OverlayObject->AssetImportData != nullptr)
	{
		Filename = OverlayObject->AssetImportData->GetFirstFilename();
	}

	FOverlaysImporter Importer;

	if (Importer.OpenFile(Filename))
	{
		TArray<FOverlayItem> NewOverlays;

		if (OverlayObject && Importer.ImportBasic(NewOverlays))
		{
			OverlayObject->Overlays = NewOverlays;
			OverlayObject->AssetImportData->Update(Filename);
			OverlayObject->MarkPackageDirty();

			ReimportResult = EReimportResult::Succeeded;
		}
	}

#endif	// WITH_EDITORONLY_DATA

	return ReimportResult;
}

int32 UReimportBasicOverlaysFactory::GetPriority() const
{
	return ImportPriority;
}
