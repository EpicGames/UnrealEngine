// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EditorReimportHandler.h"
#include "Factories/BasicOverlaysFactory.h"
#include "HAL/Platform.h"
#include "UObject/UObjectGlobals.h"

#include "ReimportBasicOverlaysFactory.generated.h"

class FString;
class UObject;

/**
 * A factory for reimporting basic overlays
 */
UCLASS(HideCategories=Object)
class UReimportBasicOverlaysFactory : public UBasicOverlaysFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	// Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	// End FReimportHandler Interface
};