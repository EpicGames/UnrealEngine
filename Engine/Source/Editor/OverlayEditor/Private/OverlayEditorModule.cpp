// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOverlayEditorModule.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "OverlayEditorPrivate.h"

DEFINE_LOG_CATEGORY(LogOverlayEditor);

/**
 * Implements the OverlayEditor module
 */
class FOverlayEditorModule
	: public IOverlayEditorModule
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override { }

	virtual void ShutdownModule() override { }

	// End IModuleInterface interface
};

IMPLEMENT_MODULE(FOverlayEditorModule, OverlayEditor);