// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * Interface for the OverlayEditor module
 */
class IOverlayEditorModule
	: public IModuleInterface
{
public:
	/** Virtual destructor */
	virtual ~IOverlayEditorModule() = default;
};
